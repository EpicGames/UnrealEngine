// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnixCommonStartup.h"
#if WITH_ENGINE
#include "Engine/Engine.h"
#endif // WITH_ENGINE
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/FeedbackContext.h"
#include "HAL/ExceptionHandling.h"
#include "Unix/UnixPlatformCrashContext.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Misc/EngineVersion.h"
#include "HAL/PlatformApplicationMisc.h"

#include <locale.h>
#include <sys/resource.h>

static FString GSavedCommandLine;
extern int32 GuardedMain( const TCHAR* CmdLine );
extern void LaunchStaticShutdownAfterError();

#if !UE_BUILD_SHIPPING && WITH_ENGINE
namespace UnixStdinTerminal
{
	FTSTicker::FDelegateHandle CurrentTicker;

	bool Tick(float DeltaTime)
	{
		static bool bInited = false;

		if (!bInited)
		{
			bInited = true;
			int flags = fcntl(STDIN_FILENO, F_GETFL);
			flags |= O_NONBLOCK;
			fcntl(STDIN_FILENO, F_SETFL, flags);
		}

		int c = getchar();
		if (c != EOF)
		{
			int CmdBufferLen = 0;
			char CmdBuffer[4096];

			CmdBuffer[CmdBufferLen++] = c;
			while (((c = getchar()) != EOF) && (CmdBufferLen < 4094))
			{
				if (c == '\n' || c == '\r')
					break;

				CmdBuffer[CmdBufferLen++] = c;
			}

			CmdBuffer[CmdBufferLen] = 0;

			UE_LOGF(LogCore, Log, "Execute Command:'%ls'\n", ANSI_TO_TCHAR(CmdBuffer));
			GEngine->DeferredCommands.Add(ANSI_TO_TCHAR(CmdBuffer));
		}

		return true;
	}
}
#endif //!UE_BUILD_SHIPPING && WITH_ENGINE

/**
 * Game-specific crash reporter
 */
void CommonUnixCrashHandler(const FGenericCrashContext& GenericContext)
{
	// at this point we should already be using malloc crash handler (see PlatformCrashHandler)

	const FUnixCrashContext& Context = static_cast< const FUnixCrashContext& >( GenericContext );
	printf("CommonUnixCrashHandler: Signal=%d\n", Context.Signal);

	/** This is the last place to gather memory stats */
	FGenericCrashContext::SetMemoryStats(FPlatformMemory::GetStats());

	// better than having mutable fields?
	const_cast< FUnixCrashContext& >(Context).CaptureStackTrace(Context.ErrorFrame);
	const_cast< FUnixCrashContext& >(Context).CaptureAllThreadContexts();
	if (GLog)
	{
		GLog->Panic();
	}
	if (GWarn)
	{
		GWarn->Flush();
	}
	if (GError)
	{
		GError->Flush();
		GError->HandleError();
	}

	return Context.GenerateCrashInfoAndLaunchReporter();
}


/**
 * Sets (soft) limit on a specific resource
 *
 * @param Resource - one of RLIMIT_* values
 * @param DesiredLimit - desired value
 * @param bIncreaseOnly - avoid changing the limit if current value is sufficient
 */
bool SetResourceLimit(int Resource, rlim_t DesiredLimit, bool bIncreaseOnly)
{
	rlimit Limit;
	if (getrlimit(Resource, &Limit) != 0)
	{
		fprintf(stderr, "getrlimit() failed with error %d (%s)\n", errno, strerror(errno));
		return false;
	}

	if (bIncreaseOnly && (Limit.rlim_cur == RLIM_INFINITY || Limit.rlim_cur >= DesiredLimit))
	{
		if (!UE_BUILD_SHIPPING)
		{
			printf("- Existing per-process limit (soft=%lu, hard=%lu) is enough for us (need only %lu)\n", Limit.rlim_cur, Limit.rlim_max, DesiredLimit);
		}
		return true;
	}

	Limit.rlim_cur = DesiredLimit;
	if (setrlimit(Resource, &Limit) != 0)
	{
		fprintf(stderr, "setrlimit() failed with error %d (%s)\n", errno, strerror(errno));

		if (errno == EINVAL)
		{
			if (DesiredLimit == RLIM_INFINITY)
			{
				fprintf(stderr, "- Max per-process value allowed is %lu (we wanted infinity).\n", Limit.rlim_max);
			}
			else
			{
				fprintf(stderr, "- Max per-process value allowed is %lu (we wanted %lu).\n", Limit.rlim_max, DesiredLimit);
			}
		}
		return false;
	}

	return true;
}

/**
 * Sets (hard) limit on a specific resource
 *
 * @param Resource - one of RLIMIT_* values
 */
static bool SetResourceMaxHardLimit(int Resource)
{
	rlimit Limit;
	if (getrlimit(Resource, &Limit) != 0)
	{
		fprintf(stderr, "getrlimit() failed with error %d (%s)\n", errno, strerror(errno));
		return false;
	}

	return SetResourceLimit(Resource, Limit.rlim_max, true);
}

/**
 * Expects GSavedCommandLine to be set up. Increases limit on
 *  - number of open files to be no less than desired (if specified on command line, otherwise left alone)
 *  - size of core file, so core gets dumped and we can debug crashed builds (unless overridden with -nocore)
 *
 */
static bool IncreasePerProcessLimits()
{
	// honor the parameter if given, else change the limit of open files to the max hard limit allowed
	int32 FileHandlesToReserve = -1;
	if (FParse::Value(*GSavedCommandLine, TEXT("numopenfiles="), FileHandlesToReserve) && FileHandlesToReserve > 0)
	{
		if (!UE_BUILD_SHIPPING)
		{
			printf("Increasing per-process limit of open file handles to %d\n", FileHandlesToReserve);
		}

		if (!SetResourceLimit(RLIMIT_NOFILE, FileHandlesToReserve, true))
		{
			fprintf(stderr, "Could not adjust number of file handles, consider changing \"nofile\" in /etc/security/limits.conf and relogin.\nerror(%d): %s\n", errno, strerror(errno));
			return false;
		}
	}
	else
	{
		// Set the highest value we can for number of files open
		SetResourceMaxHardLimit(RLIMIT_NOFILE);
	}

	// core dump policy:
	// - Shipping disables it by default (unless -core is passed)
	// - The rest set it to infinity unless -nocore is passed
	// (in all scenarios user wish as expressed with -core or -nocore takes priority)
	// Note that we used to have Test disable cores by default too. This has been changed around UE 4.15.
	// Since 4.20, inability to change the limit is no longer a failure unless switches were used
	bool bFailIfUnableToChange = false;
	bool bDisableCore = (UE_BUILD_SHIPPING != 0);
	if (FParse::Param(*GSavedCommandLine, TEXT("nocore")))
	{
		bDisableCore = true;
		bFailIfUnableToChange = true;
	}
	if (FParse::Param(*GSavedCommandLine, TEXT("core")))
	{
		bDisableCore = false;
		bFailIfUnableToChange = true;
	}

	if (bDisableCore)
	{
		printf("Disabling core dumps.\n");
		if (!SetResourceLimit(RLIMIT_CORE, 0, false) && bFailIfUnableToChange)
		{
			fprintf(stderr, "Could not set core file size to 0, error(%d): %s\n", errno, strerror(errno));
			return false;
		}
	}
	else
	{
		printf("Increasing per-process limit of core file size to infinity.\n");
		if (!SetResourceLimit(RLIMIT_CORE, RLIM_INFINITY, true) && bFailIfUnableToChange)
		{
			fprintf(stderr, "Could not adjust core file size, consider changing \"core\" in /etc/security/limits.conf and relogin.\nerror(%d): %s\n", errno, strerror(errno));
			fprintf(stderr, "Alternatively, pass -nocore if you are unable or unwilling to do that.\n");
			return false;
		}
	}

	if constexpr (WITH_PROCESS_PRIORITY_CONTROL)
	{
		printf("Increasing per-process limit for scheduling priority.\n");
		SetResourceMaxHardLimit(RLIMIT_NICE);
	}

	return true;
}

int CommonUnixMain(int argc, char *argv[], int (*RealMain)(const TCHAR * CommandLine), void (*AppExitCallback)())
{
	FString EarlyInitCommandLine;
	FPlatformApplicationMisc::EarlyUnixInitialization(EarlyInitCommandLine);
	GSavedCommandLine += EarlyInitCommandLine;

	FPlatformMisc::SetGracefulTerminationHandler();

	if (UE_BUILD_SHIPPING)
	{
		// only printed in shipping
		printf("%s %d %d\n", StringCast<ANSICHAR>(*FEngineVersion::Current().ToString()).Get(), GPackageFileUEVersion.ToValue(), GPackageFileLicenseeUEVersion);
	}

	int ErrorLevel = 0;

	if (setenv("LC_NUMERIC", "en_US", 1) != 0)
	{
		int ErrNo = errno;
		fprintf(stderr, "Unable to setenv(LC_NUMERIC): errno=%d (%s)", ErrNo, strerror(ErrNo));
	}

	for (int32 Option = 1; Option < argc; Option++)
	{
		GSavedCommandLine += TEXT(" ");
		// we need to quote stuff that has spaces in it because something somewhere is removing quotation marks before they arrive here
		FString Temp = UTF8_TO_TCHAR(argv[Option]);
		if (Temp.Contains(TEXT(" ")))
		{
			int32 Quote = 0;
			if(Temp.StartsWith(TEXT("-")))
			{
				int32 Separator;
				if (Temp.FindChar('=', Separator))
				{
					Quote = Separator + 1;
				}
			}
			Temp = Temp.Left(Quote) + TEXT("\"") + Temp.Mid(Quote) + TEXT("\"");
		}
		GSavedCommandLine += Temp; 	// note: technically it depends on locale
	}

	if (!UE_BUILD_SHIPPING)
	{
		GAlwaysReportCrash = true;	// set by default and reverse the behavior
		if ( FParse::Param( *GSavedCommandLine,TEXT("nocrashreports") ) || FParse::Param( *GSavedCommandLine,TEXT("no-crashreports") ) )
		{
			GAlwaysReportCrash = false;
		}
	}

	if (FPlatformApplicationMisc::ShouldIncreaseProcessLimits() && !IncreasePerProcessLimits())
	{
		fprintf(stderr, "Could not set desired per-process limits, consider changing system limits.\n");
		ErrorLevel = 1;
	}
	else
	{
#if UE_BUILD_DEBUG
		if( true && !GAlwaysReportCrash )
#else
		if( FPlatformMisc::IsDebuggerPresent() && !GAlwaysReportCrash )
#endif
		{
			// Don't use exception handling when a debugger is attached to exactly trap the crash. This does NOT check
			// whether we are the first instance or not!
			ErrorLevel = RealMain( *GSavedCommandLine );
		}
		else
		{
#if !UE_BUILD_SHIPPING && WITH_ENGINE
			bool bReadCommandsFromStdin = FParse::Param(*GSavedCommandLine, TEXT("cmdstdin"));
			if (bReadCommandsFromStdin)
			{
				UnixStdinTerminal::CurrentTicker = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateStatic(&UnixStdinTerminal::Tick), 1.0f);
			}
#endif

			FPlatformMisc::SetCrashHandler(CommonUnixCrashHandler);
			GIsGuarded = 1;
			// Run the guarded code.
			ErrorLevel = RealMain( *GSavedCommandLine );
			GIsGuarded = 0;

#if !UE_BUILD_SHIPPING && WITH_ENGINE
			if (bReadCommandsFromStdin)
			{
				FTSTicker::RemoveTicker(UnixStdinTerminal::CurrentTicker);
			}
#endif
		}
	}

	// Final shut down.
	if (AppExitCallback)
	{
		// Workaround function to avoid circular dependencies between Launch and CommonUnixStartup modules.

		// Other platforms call FEngineLoop::AppExit() in their main() (removed by preprocessor if compiled without engine), 
		// but on Unix we want to share a common main() in CommonUnixStartup module, so not just the engine but all the programs 
		// could share this logic. Unfortunately, AppExit() practice breaks this nice approach since FEngineLoop cannot be moved outside of 
		// Launch module without making too many changes. Hence CommonUnixMain will call it through this function if provided.

		AppExitCallback();
	}

	// check if a specific return code has been set
	uint8 OverriddenErrorLevel = 0;
	if (FPlatformMisc::HasOverriddenReturnCode(&OverriddenErrorLevel))
	{
		ErrorLevel = OverriddenErrorLevel;
	}

	if (ErrorLevel)
	{
		printf("Exiting abnormally (error code: %d)\n", ErrorLevel);
	}
	return ErrorLevel;
}


class FUnixCommonStartupModule : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override {};
	virtual void ShutdownModule() override {};
};

IMPLEMENT_MODULE(FUnixCommonStartupModule, UnixCommonStartup);
