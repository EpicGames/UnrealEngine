// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"

/**
 * @brief CommonUnixMain - executes common startup code for Unix programs/engine
 * @param argc - number of arguments in argv[]
 * @param argv - array of arguments
 * @param RealMain - the next main routine to call in chain
 * @param AppExitCallback - workaround for Launch module that needs to call FEngineLoop::AppExit() at certain point
 * @return error code to return to the OS
 */
int UNIXCOMMONSTARTUP_API CommonUnixMain(int argc, char *argv[], int (*RealMain)(const TCHAR * CommandLine), void (*AppExitCallback)() = nullptr);
