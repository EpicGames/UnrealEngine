// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class UnixCommonStartup : ModuleRules
{
	public UnixCommonStartup(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("ApplicationCore");
		PrivateDependencyModuleNames.Add("Projects");

		PrivateDependencyModuleNames.Add("Slate");

		// Only include Engine if necessary
		if (Target.bCompileAgainstEngine)
		{
			PublicDependencyModuleNames.Add("Engine");
		}

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"SourceControl",
				}
			);
		}
	}
}
